// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BuildStorageTool : ModuleRules
{
	public BuildStorageTool(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = true;
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Analytics",
				"ApplicationCore",
				"Core",
				"DesktopPlatform",
				"Json",
				"OutputLog",
				"PakFile",
				"Projects",
				"Slate",
				"SlateCore",
				"Sockets",
				"StandaloneRenderer",
				"StorageServerWidgets",
				"StudioTelemetry",
				"ToolWidgets",
				"Zen",
			});

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnixCommonStartup"
				}
			);
		}

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Launch",
			});

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/crashpad_handler.exe");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zen.exe");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zen.pdb");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zenserver.exe");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zenserver.pdb");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/win-x64/OidcToken.exe");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Linux/zen");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Linux/zenserver");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/linux-x64/OidcToken");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/crashpad_handler");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/zen");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/zenserver");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/osx-x64/OidcToken");
		}

		RuntimeDependencies.Add("$(EngineDir)/Content/Editor/Slate/...*.svg", StagedFileType.UFS);

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			RuntimeDependencies.Add("$(EngineDir)/Content/SlateFileDialogs/...*.png", StagedFileType.UFS);
		}
	}
}
