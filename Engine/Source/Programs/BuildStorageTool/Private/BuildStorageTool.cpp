// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildStorageTool.h"
#include "Experimental/ZenServerInterface.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CompilationResult.h"
#include "Misc/MessageDialog.h"
#include "Misc/Optional.h"
#include "Misc/Timespan.h"
#include "Modules/BuildVersion.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RequiredProgramMainCPPInclude.h"
#include "SBuildActivity.h"
#include "SBuildSelection.h"
#include "SBuildLogin.h"
#include "SMessageDialog.h"
#include "StandaloneRenderer.h"
#include "BuildStorageToolStyle.h"
#include "BuildStorageToolHelpWidget.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#include "ZenServiceInstanceManager.h"
#include "BuildServiceInstanceManager.h"
#include "Version/AppVersionDefines.h"
#include "Parameters/BuildStorageToolParametersBuilder.h"
#include "OutputLogCreationParams.h"
#include "OutputLogModule.h"
#include "OutputLogSettings.h"
#include "StudioTelemetry.h"
#include "ZenBuildUtils.h"
#include "BuildStorageToolIPC.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/Commands/Commands.h"
#include "HAL/PlatformApplicationMisc.h"
#include "String/ParseTokens.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"


#if PLATFORM_WINDOWS
#include "Runtime/Launch/Resources/Windows/Resource.h"
#include "Windows/WindowsApplication.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <shellapi.h>
#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_LINUX
#include "UnixCommonStartup.h"
#elif PLATFORM_MAC
#include "Mac/MacProgramDelegate.h"
#include "LaunchEngineLoop.h"
#else
#error "Unsupported platform!"
#endif

#define LOCTEXT_NAMESPACE "BuildStorageTool"

// These macros are not properly defined by UBT in the case of an engine program with bTreatAsEngineModule=true
// So define them here as a workaround
#define IMPLEMENT_ENCRYPTION_KEY_REGISTRATION()
#define IMPLEMENT_SIGNING_KEY_REGISTRATION()
IMPLEMENT_APPLICATION(BuildStorageTool, "BuildStorageTool");

DEFINE_LOG_CATEGORY(LogBuildStorageTool);

static void OnRequestExit()
{
	RequestEngineExit(TEXT("BuildStorageTool Closed"));
}

static void HideOnCloseOverride(const TSharedRef<SWindow>& WindowBeingClosed)
{
	WindowBeingClosed->HideWindow();
}

class FBuildStorageToolCommands : public TCommands<FBuildStorageToolCommands>
{
public:
	FBuildStorageToolCommands()
		: TCommands<FBuildStorageToolCommands>(
			TEXT("BuildStorageTool"),
			LOCTEXT("BuildStorageToolCommands", "Build Storage Tool"),
			NAME_None,
			FCoreStyle::Get().GetStyleSetName())
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(GoToBuild, "Go to Build...", "Navigate to a build by entering its download URL", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::G));
	}

	TSharedPtr<FUICommandInfo> GoToBuild;
};

class FBuildStorageToolPasteHandler : public IInputProcessor
{
public:
	void SetBuildSelection(TSharedPtr<SBuildSelection> InBuildSelection)
	{
		BuildSelection = InBuildSelection;
	}

	virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {}

	virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override
	{
		if (!(InKeyEvent.IsControlDown() || InKeyEvent.IsCommandDown()) || InKeyEvent.GetKey() != EKeys::V)
		{
			return false;
		}

		if (SlateApp.GetActiveModalWindow().IsValid())
		{
			return false;
		}

		TSharedPtr<SBuildSelection> PinnedBuildSelection = BuildSelection.Pin();
		if (!PinnedBuildSelection.IsValid())
		{
			return false;
		}

		FString ClipboardText;
		FPlatformApplicationMisc::ClipboardPaste(ClipboardText);
		UE::Zen::Build::FBuildReference BuildReference;
		if (UE::Zen::Build::TryParseBuildReferenceFromUrl(ClipboardText, BuildReference))
		{
			PinnedBuildSelection->ActOnBuildArtifact(SBuildSelection::EBuildArtifactAction::Highlight, BuildReference);
			return true;
		}

		return false;
	}

	virtual const TCHAR* GetDebugName() const override { return TEXT("BuildStorageToolPasteHandler"); }

private:
	TWeakPtr<SBuildSelection> BuildSelection;
};

class FBuildStorageToolApp
{
	void ExitTool()
	{
		FSlateApplication::Get().RequestDestroyWindow(Window.ToSharedRef());
	}

	void OnAboutCommandPressed()
	{
		FSlateApplication& SlateApplicaton = FSlateApplication::Get();

		if (SlateApplicaton.GetActiveModalWindow() != nullptr)
		{
			return;
		}

		TSharedPtr<SWidget> ParentWidget = SlateApplicaton.GetUserFocusedWidget(0);
		if (!ensure(ParentWidget))
		{
			return;
		}

		// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
		const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
		FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);
		FVector2D AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition(Anchor, FVector2D(441, 537), true, FVector2D::ZeroVector, Orient_Horizontal);

		TSharedPtr<SWindow> WindowDialog = SNew(SWindow)
			.AutoCenter(EAutoCenter::None)
			.ScreenPosition(AdjustedSummonLocation)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::Autosized)
			.Title(LOCTEXT("HelpAboutHeader", "About"));

		WindowDialog->SetContent(SNew(SBuildStorageToolHelpWidget)
			.ToolParameters(&ToolParameters)
		);

		FSlateApplication::Get().AddModalWindow(WindowDialog.ToSharedRef(), ParentWidget);
	}

	void GoToBuild()
	{
		FSlateApplication& SlateApp = FSlateApplication::Get();

		if (SlateApp.GetActiveModalWindow() != nullptr)
		{
			return;
		}

		TSharedPtr<SWindow> DialogWindow;
		bool bConfirmed = false;

		TSharedRef<SEditableTextBox> UrlTextBox = SNew(SEditableTextBox)
			.HintText(LOCTEXT("GoToBuildHint", "Enter a build download URL..."))
			.OnTextCommitted_Lambda([&bConfirmed, &DialogWindow](const FText&, ETextCommit::Type CommitType)
			{
				if (CommitType == ETextCommit::OnEnter)
				{
					bConfirmed = true;
					if (DialogWindow.IsValid())
					{
						DialogWindow->RequestDestroyWindow();
					}
				}
			});

		DialogWindow = SNew(SWindow)
			.Title(LOCTEXT("GoToBuildTitle", "Go to Build"))
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::Autosized)
			[
				SNew(SBox)
				.MinDesiredWidth(500.0f)
				.MaxDesiredWidth(500.0f)
				.Padding(FMargin(16.0f))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 0, 0, 8)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("GoToBuildLabel", "Enter the download URL of the build to navigate to:"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 0, 0, 16)
					[
						UrlTextBox
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Right)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 0, 8, 0)
						[
							SNew(SButton)
							.Text(LOCTEXT("GoButton", "Go"))
							.OnClicked_Lambda([&bConfirmed, &DialogWindow]()
							{
								bConfirmed = true;
								DialogWindow->RequestDestroyWindow();
								return FReply::Handled();
							})
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SButton)
							.Text(LOCTEXT("CancelButton", "Cancel"))
							.OnClicked_Lambda([&DialogWindow]()
							{
								DialogWindow->RequestDestroyWindow();
								return FReply::Handled();
							})
						]
					]
				]
			];

		SlateApp.AddModalWindow(DialogWindow.ToSharedRef(), Window);

		if (bConfirmed)
		{
			FString Url = UrlTextBox->GetText().ToString();
			UE::Zen::Build::FBuildReference BuildReference;
			if (UE::Zen::Build::TryParseBuildReferenceFromUrl(Url, BuildReference))
			{
				BuildSelection->ActOnBuildArtifact(SBuildSelection::EBuildArtifactAction::Highlight, BuildReference);
			}
		}
	}

	bool CanExecuteExclusiveAction()
	{
		return !bLatentExclusiveOperationActive;
	}

	TSharedRef< SWidget > MakeMainMenu()
	{
		FMenuBarBuilder MenuBuilder( CommandList );
		{
			MenuBuilder.AddPullDownMenu(
				LOCTEXT( "FileMenu", "File" ),
				LOCTEXT( "FileMenu_ToolTip", "Opens the file menu" ),
				FOnGetContent::CreateRaw( this, &FBuildStorageToolApp::FillFileMenu ) );

			MenuBuilder.AddPullDownMenu(
				LOCTEXT("ToolsMenu", "Tools"),
				LOCTEXT("ToolsMenu_ToolTip", "Opens the tools menu"),
				FOnGetContent::CreateRaw(this, &FBuildStorageToolApp::FillToolsMenu) );

			MenuBuilder.AddPullDownMenu(
				LOCTEXT( "HelpMenu", "Help" ),
				LOCTEXT( "HelpMenu_ToolTip", "Opens the help menu" ),
				FOnGetContent::CreateRaw( this, &FBuildStorageToolApp::FillHelpMenu ) );
		}

		// Create the menu bar
		TSharedRef< SWidget > MenuBarWidget = MenuBuilder.MakeWidget();
		MenuBarWidget->SetVisibility( EVisibility::Visible ); // Work around for menu bar not showing on Mac

		return MenuBarWidget;
	}

	void ExploreLogDirectory()
	{
		FString Path = FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir());
		if (!Path.Len() || !IFileManager::Get().DirectoryExists(*Path))
		{
			return;
		}

		FPlatformProcess::ExploreFolder(*FPaths::GetPath(Path));
	}

	void OpenLogWindow()
	{
		/*** Output Log Widget ***/
		FOutputLogModule& OutputLogModule = FModuleManager::Get().LoadModuleChecked<FOutputLogModule>("OutputLog");

		// hide the debug console
		IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("OutputLogModule.HideConsole"));
		if(ensure(CVar))
		{
			CVar->Set(true);
		}

		// setup OutputLog settings
		UOutputLogSettings* Settings = GetMutableDefault<UOutputLogSettings>();
		if(Settings)
		{
			Settings->CategoryColorizationMode = ELogCategoryColorizationMode::ColorizeWholeLine;
		}

		// Open new slate window
		FSlateApplication::Get().AddWindowAsNativeChild(
			SNew(SWindow)
			.Title(LOCTEXT("LogWindowTitle", "Build Storage Tool Log"))
			.ClientSize(FVector2D(1200.0f, 600.0f))
			.ActivationPolicy(EWindowActivationPolicy::Always)
			.SizingRule(ESizingRule::UserSized)
			.IsTopmostWindow(false)
			.FocusWhenFirstShown(false)
			.SupportsMaximize(true)
			.SupportsMinimize(true)
			.HasCloseButton(true)
			[
				OutputLogModule.MakeOutputLogWidget(FOutputLogCreationParams())
			],
			Window.ToSharedRef()
		);
	}

	TSharedRef<SWidget> FillToolsMenu()
	{
		const bool bCloseSelfOnly = false;
		const bool bSearchable = false;
		const bool bRecursivelySearchable = false;

		FMenuBuilder MenuBuilder(true,
			CommandList,
			TSharedPtr<FExtender>(),
			bCloseSelfOnly,
			&FCoreStyle::Get(),
			bSearchable,
			NAME_None,
			bRecursivelySearchable);

		MenuBuilder.AddMenuEntry(FBuildStorageToolCommands::Get().GoToBuild);

		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddMenuEntry(
					LOCTEXT("ExploreLogDirectory", "Explore Log Directory"),
					LOCTEXT("ExploreLogDirectory_ToolTip", "Explore the directory containing log files"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw( this, &FBuildStorageToolApp::ExploreLogDirectory )
					),
					NAME_None,
					EUserInterfaceActionType::Button
				);

		MenuBuilder.AddMenuEntry(
					LOCTEXT("OpenLogWindow", "Open Log Window"),
					LOCTEXT("OpenLogWindow_ToolTip", "Opens the Log Window"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateRaw( this, &FBuildStorageToolApp::OpenLogWindow )
					),
					NAME_None,
					EUserInterfaceActionType::Button
				);

		MenuBuilder.BeginSection(TEXT("TransferOptions"), LOCTEXT("TransferOptionsMenu", "Transfer Options"));
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("HostOverride", "Host Override"),
				LOCTEXT("HostOverride_ToolTip", "Pick a host to use for build transfers instead of automatic selection of the nearest host"),
				FNewMenuDelegate::CreateRaw(this, &FBuildStorageToolApp::OpenHostOverrideMenu));

			MenuBuilder.AddMenuEntry(
				LOCTEXT("AppendFiles", "Append Files Only"),
				LOCTEXT("AppendFiles_ToolTip", "When downloading files to a directory, only append new files, don't remove files that used to be part of a build in that directory"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(
						this, &FBuildStorageToolApp::ToggleAppendFiles),
					FCanExecuteAction(),
					FIsActionChecked::CreateRaw(
						this, &FBuildStorageToolApp::IsAppendFilesSelected)
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("CleanDestination", "Clean Extra Files"),
				LOCTEXT("CleanDestination_ToolTip", "When downloading files to a directory, any extra files (not belonging to a past build) that existed in that directory will be deleted"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(
						this, &FBuildStorageToolApp::ToggleCleanDestination),
					FCanExecuteAction(),
					FIsActionChecked::CreateRaw(
						this, &FBuildStorageToolApp::IsCleanDestinationSelected)
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ForceRedownload", "Force Redownload"),
				LOCTEXT("ForceRedownload_ToolTip", "Forces downloads to get all data not just the data that is not already there"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(
						this, &FBuildStorageToolApp::ToggleForceRedownload),
					FCanExecuteAction(),
					FIsActionChecked::CreateRaw(
						this, &FBuildStorageToolApp::IsForceRedownloadSelected)
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("EnableScavenging", "Enable Scavenging"),
				LOCTEXT("EnableScavenging_ToolTip", "Allows downloads to find and use data from downloads elsewhere on your machine"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(
						this, &FBuildStorageToolApp::ToggleScavenging),
					FCanExecuteAction::CreateRaw(
						this, &FBuildStorageToolApp::CanSelectScavenging),
					FIsActionChecked::CreateRaw(
						this, &FBuildStorageToolApp::IsScavengingSelected)
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("EnableVerify", "Enable Verification"),
				LOCTEXT("EnableVerify_ToolTip", "Performs a verification pass on data after it has been downloaded"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(
						this, &FBuildStorageToolApp::ToggleVerify),
					FCanExecuteAction(),
					FIsActionChecked::CreateRaw(
						this, &FBuildStorageToolApp::IsVerifySelected)
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("BoostWorkers", "Boost Downloads"),
				LOCTEXT("BoostWorkers_ToolTip", "Increases resource usage when downloading, computer may be less responsive"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(
						this, &FBuildStorageToolApp::ToggleBoostWorkers),
					FCanExecuteAction(),
					FIsActionChecked::CreateRaw(
						this, &FBuildStorageToolApp::IsBoostWorkersSelected)
				),
				NAME_None,
				EUserInterfaceActionType::Check
			);
		}
		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}

	void OpenHostOverrideMenu(FMenuBuilder& MenuBuilder)
	{
		using namespace UE::Zen::Build;
		const FServiceSettings& ServiceSettings = BuildServiceInstanceManager->GetBuildServiceInstance()->GetSettings();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("NoneHostOverride", "None"),
			LOCTEXT("NoneHostOverride_ToolTip", "Disables host overriding and restores automatic selection of nearest host when downloading"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FBuildStorageToolApp::SetHostOverride, FString()),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FBuildStorageToolApp::CompareHostOverride, FString())
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		TArray<FStringView> HostCandidates;
		UE::String::ParseTokens(ServiceSettings.GetHost(), ';', [&HostCandidates](FStringView Host)
		{
			HostCandidates.Emplace(Host);
		}, UE::String::EParseTokensOptions::SkipEmpty | UE::String::EParseTokensOptions::Trim);

		for (FStringView HostCandidate : HostCandidates)
		{
			MenuBuilder.AddMenuEntry(
				FText::FromStringView(HostCandidate),
				FText(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(this, &FBuildStorageToolApp::SetHostOverride, FString(HostCandidate)),
					FCanExecuteAction(),
					FIsActionChecked::CreateRaw(this, &FBuildStorageToolApp::CompareHostOverride, FString(HostCandidate))
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}
	}

	TSharedRef<SWidget> FillFileMenu()
	{
		const bool bCloseSelfOnly = false;
		const bool bSearchable = false;
		const bool bRecursivelySearchable = false;

		FMenuBuilder MenuBuilder(true,
			nullptr,
			TSharedPtr<FExtender>(),
			bCloseSelfOnly,
			&FCoreStyle::Get(),
			bSearchable,
			NAME_None,
			bRecursivelySearchable);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Exit", "Exit"),
			LOCTEXT("Exit_ToolTip", "Exits the Storage Server UI"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw( this, &FBuildStorageToolApp::ExitTool ),
				FCanExecuteAction::CreateRaw(this, &FBuildStorageToolApp::CanExecuteExclusiveAction)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FillHelpMenu()
	{
		const bool bCloseSelfOnly = false;
		const bool bSearchable = false;
		const bool bRecursivelySearchable = false;

		FMenuBuilder MenuBuilder(true,
			nullptr,
			TSharedPtr<FExtender>(),
			bCloseSelfOnly,
			&FCoreStyle::Get(),
			bSearchable,
			NAME_None,
			bRecursivelySearchable);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("About", "About"),
			LOCTEXT("About_ToolTip", "Show the about dialog"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw( this, &FBuildStorageToolApp::OnAboutCommandPressed ),
				FCanExecuteAction::CreateRaw(this, &FBuildStorageToolApp::CanExecuteExclusiveAction)
			),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		return MenuBuilder.MakeWidget();
	}

	void OnIPCMessageReceived(const UE::BuildStorageTool::IMessage& Message)
	{
		using namespace UE::BuildStorageTool;

		if (Message.GetMessageType() == FUrlMessage::MessageType)
		{
			const FUrlMessage& UrlMessage = static_cast<const FUrlMessage&>(Message);
			UE::Zen::Build::FBuildReference BuildReference;
			if (TryParseBuildReferenceFromUrl(UrlMessage.Url, BuildReference))
			{
				SBuildSelection::EBuildArtifactAction Action = UrlMessage.Action == EUrlAction::HighlightAndDownload
					? SBuildSelection::EBuildArtifactAction::HighlightAndDownload
					: SBuildSelection::EBuildArtifactAction::Highlight;

				AsyncTask(ENamedThreads::GameThread, [this, BuildReference = MoveTemp(BuildReference), Action]()
				{
					Window->HACK_ForceToFront();
					Window->DrawAttention(FWindowDrawAttentionParameters());
					BuildSelection->ActOnBuildArtifact(Action, BuildReference);
				});
			}
		}
	}

public:
	FBuildStorageToolApp(FSlateApplication& InSlate, const FBuildStorageToolParameters& InToolParameters)
		: Slate(InSlate), ToolParameters(InToolParameters)
	{
		ZenServiceInstanceManager = MakeShared<UE::Zen::FServiceInstanceManager>();
		BuildServiceInstanceManager = MakeShared<UE::Zen::Build::FServiceInstanceManager>();

		FBuildStorageToolCommands::Register();
		CommandList = MakeShared<FUICommandList>();
		CommandList->MapAction(
			FBuildStorageToolCommands::Get().GoToBuild,
			FExecuteAction::CreateRaw(this, &FBuildStorageToolApp::GoToBuild));

		InstallMessageDialogOverride();
		MessageListenServer.OnMessageReceived.AddRaw(this, &FBuildStorageToolApp::OnIPCMessageReceived);
		MessageListenServer.Start();
	}

	virtual ~FBuildStorageToolApp()
	{
		MessageListenServer.ShutdownJoin();
		RemoveMessageDialogOverride();
		FBuildStorageToolCommands::Unregister();
	}

	void Run()
	{
		STUDIO_TELEMETRY_SESSION_SCOPE

		Window =
			SNew(SWindow)
			.Title(GetWindowTitle())
			.ClientSize(FVector2D(1280.0f, 720.0f))
			.ActivationPolicy(EWindowActivationPolicy::Always)
			.SizingRule(ESizingRule::UserSized)
			.IsTopmostWindow(false)
			.FocusWhenFirstShown(true)
			.SupportsMaximize(true)
			.SupportsMinimize(true)
			.HasCloseButton(true)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					// Menu
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						MakeMainMenu()
					]

					// Login panel
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 10.0f, 5.0f, 0.0f)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						SNew(SBuildLogin)
						.ZenServiceInstance(ZenServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						.BuildServiceInstance(BuildServiceInstanceManager.ToSharedRef(), &UE::Zen::Build::FServiceInstanceManager::GetBuildServiceInstance)
					]

					// Selection panel
					+ SVerticalBox::Slot()
					.Padding(0.0f, 10.0f, 5.0f, 10.0f)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(BuildSelection, SBuildSelection)
						.ZenServiceInstance(ZenServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						.BuildServiceInstance(BuildServiceInstanceManager.ToSharedRef(), &UE::Zen::Build::FServiceInstanceManager::GetBuildServiceInstance)
						.PreBuildTransfer_Lambda([this]
							(FString& InHostOverride, UE::Zen::Build::EBuildTransferRequestFlags& RequestFlags)
							{
								InHostOverride = HostOverride;
								RequestFlags = UE::Zen::Build::EBuildTransferRequestFlags::None;
								if (bAppendFiles)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::Append;
								}
								if (bCleanDestination)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::Clean;
								}
								if (bForceRedownload)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::Force;
								}
								else if (bScavenging)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::Scavenge;
								}
								if (bVerify)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::Verify;
								}
								if (bBoostWorkers)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::BoostWorkers;
								}
							})
						.PreOplogBuildTransfer_Lambda([this]
							(FString& InHostOverride, UE::Zen::Build::EBuildTransferRequestFlags& RequestFlags)
							{
								InHostOverride = HostOverride;
								RequestFlags = UE::Zen::Build::EBuildTransferRequestFlags::None;
								// We want oplogs to always download "clean"
								RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::Clean;
								if (bForceRedownload)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::Force;
								}
								if (bBoostWorkers)
								{
									RequestFlags |= UE::Zen::Build::EBuildTransferRequestFlags::BoostWorkers;
								}
							})
						.OnBuildTransferStarted_Lambda([this]
							(UE::Zen::Build::FBuildServiceInstance::FBuildTransfer Transfer, FStringView Name, FStringView Platform)
							{
								BuildActivity->AddBuildTransfer(Transfer, Name, Platform);
							})
					]

					// Activity panel
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f, 10.0f, 5.0f, 0.0f)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Bottom)
					[
						SAssignNew(BuildActivity, SBuildActivity)
						.ZenServiceInstance(ZenServiceInstanceManager.ToSharedRef(), &UE::Zen::FServiceInstanceManager::GetZenServiceInstance)
						.BuildServiceInstance(BuildServiceInstanceManager.ToSharedRef(), &UE::Zen::Build::FServiceInstanceManager::GetBuildServiceInstance)
					]
				]
			];

		Slate.AddWindow(Window.ToSharedRef(), true);

		// If the tool was launched with a Url argument, attempt to download or highlight that artifact
		FString Url;
		if (FParse::Value(FCommandLine::Get(), TEXT("-DownloadBuild"), Url))
		{
			UE::Zen::Build::FBuildReference BuildReference;
			if (UE::Zen::Build::TryParseBuildReferenceFromUrl(Url, BuildReference))
			{
				BuildSelection->ActOnBuildArtifact(
					SBuildSelection::EBuildArtifactAction::HighlightAndDownload,
					BuildReference,
					false);
			}
		}
		else if (FParse::Value(FCommandLine::Get(), TEXT("-GoToBuild"), Url))
		{
			UE::Zen::Build::FBuildReference BuildReference;
			if (UE::Zen::Build::TryParseBuildReferenceFromUrl(Url, BuildReference))
			{
				BuildSelection->ActOnBuildArtifact(
					SBuildSelection::EBuildArtifactAction::Highlight,
					BuildReference,
					false);
			}
		}

		// Register input pre-processor for paste-to-navigate
		TSharedRef<FBuildStorageToolPasteHandler> PasteHandler = MakeShared<FBuildStorageToolPasteHandler>();
		PasteHandler->SetBuildSelection(BuildSelection);
		Slate.RegisterInputPreProcessor(PasteHandler);

		// Setting focus seems to have to happen after the Window has been added
		Slate.ClearKeyboardFocus(EFocusCause::Cleared);

		double DeltaTime = 0.0;
		double LastTime = FPlatformTime::Seconds();
		const float IdealFrameTime = 1.0f / 60;
		const float BackgroundFrameTime = 1.0f / 4;
		
		// loop until the app is ready to quit
		while (!IsEngineExitRequested())
		{
			BeginExitIfRequested();

			FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
			FTSTicker::GetCoreTicker().Tick(DeltaTime);
			Slate.PumpMessages();
			Slate.Tick();

			const float FrameTime = !FPlatformApplicationMisc::IsThisApplicationForeground() && (FPlatformTime::Seconds() - FSlateApplication::Get().GetLastUserInteractionTime()) > 5 ? BackgroundFrameTime : IdealFrameTime;
			FPlatformProcess::Sleep(FMath::Max<float>(0.f, FrameTime - static_cast<float>(FPlatformTime::Seconds() - LastTime)));

			DeltaTime = FPlatformTime::Seconds() - LastTime;
			LastTime = FPlatformTime::Seconds();

			UE::Stats::FStats::AdvanceFrame(false);
			FCoreDelegates::OnEndFrame.Broadcast();

			GFrameCounter++;
		}

		Slate.UnregisterInputPreProcessor(PasteHandler);

		// Make sure the window is hidden, because it might take a while for the background thread to finish.
		Window->HideWindow();
	}

private:
	FText GetWindowTitle()
	{
		FText ToolName = LOCTEXT("WindowTitle", "Unreal Build Storage Tool");
#if defined(BUILD_STORAGE_TOOL_CHANGELIST_STRING)
		return FText::Format(LOCTEXT("WindowTitle_VersionFormat", "{0} ({1})"), ToolName, FText::FromString(BUILD_STORAGE_TOOL_CHANGELIST_STRING));
#else
		return ToolName;
#endif
	}

	EAppReturnType::Type OnModalMessageDialog(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessage, const FText& InText, const FText& InTitle)
	{
		if (IsInGameThread() && FSlateApplication::IsInitialized() && FSlateApplication::Get().CanAddModalWindow())
		{
			return OpenModalMessageDialog_Internal(InMessageCategory, InMessage, InText, InTitle, Window);
		}
		else
		{
			return FPlatformMisc::MessageBoxExt(InMessage, *InText.ToString(), *InTitle.ToString());
		}
	}

	void InstallMessageDialogOverride()
	{
		FCoreDelegates::ModalMessageDialog.BindRaw(this, &FBuildStorageToolApp::OnModalMessageDialog);
	}

	void RemoveMessageDialogOverride()
	{
		FCoreDelegates::ModalMessageDialog.Unbind();
	}

	void ToggleAppendFiles()
	{
		bAppendFiles = !bAppendFiles;
	}

	bool IsAppendFilesSelected()
	{
		return bAppendFiles;
	}

	void ToggleCleanDestination()
	{
		bCleanDestination = !bCleanDestination;
	}

	bool IsCleanDestinationSelected()
	{
		return bCleanDestination;
	}

	void ToggleForceRedownload()
	{
		bForceRedownload = !bForceRedownload;
	}

	bool IsForceRedownloadSelected()
	{
		return bForceRedownload;
	}

	void ToggleScavenging()
	{
		bScavenging = !bScavenging;
	}

	bool CanSelectScavenging()
	{
		return !bForceRedownload;
	}

	bool IsScavengingSelected()
	{
		return bScavenging && !bForceRedownload;
	}

	void ToggleVerify()
	{
		bVerify = !bVerify;
	}

	bool IsVerifySelected()
	{
		return bVerify;
	}

	void ToggleBoostWorkers()
	{
		bBoostWorkers = !bBoostWorkers;
	}

	bool IsBoostWorkersSelected()
	{
		return bBoostWorkers;
	}

	void SetHostOverride(FString InHostOverride)
	{
		HostOverride = InHostOverride;
	}

	bool CompareHostOverride(FString InHostOverride) const
	{
		return HostOverride == InHostOverride;
	}

	FCriticalSection CriticalSection;
	UE::BuildStorageTool::FMessageListenServer MessageListenServer;
	FSlateApplication& Slate;
	TSharedPtr<SWindow> Window;
	TSharedPtr<SNotificationItem> CompileNotification;
	TSharedPtr<UE::Zen::FServiceInstanceManager> ZenServiceInstanceManager;
	TSharedPtr<UE::Zen::Build::FServiceInstanceManager> BuildServiceInstanceManager;
	const FBuildStorageToolParameters& ToolParameters;
	FString HostOverride;
	bool bAppendFiles = false;
	bool bCleanDestination = false;
	bool bForceRedownload = false;
	bool bScavenging = true;
	bool bVerify = false;
	bool bBoostWorkers = false;

	TSharedPtr<FUICommandList> CommandList;
	std::atomic<bool> bLatentExclusiveOperationActive = false;
	TSharedPtr<SBuildSelection> BuildSelection;
	TSharedPtr<SBuildActivity> BuildActivity;
};

int BuildStorageToolMain(const TCHAR* CmdLine)
{
	using namespace UE::BuildStorageTool;

	ON_SCOPE_EXIT
	{
		RequestEngineExit(TEXT("Exiting"));
		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();
	};

	// start up the main loop
	GEngineLoop.PreInit(CmdLine);

	// ensure that the backlog is enabled
	if(GLog)
	{
		GLog->EnableBacklog(true);
	}

	FSystemWideCriticalSection SystemWideBuildStorageToolCritSec(TEXT("BuildStorageTool"));
	if (!SystemWideBuildStorageToolCritSec.IsValid())
	{
		FString Url;
		if (FParse::Value(FCommandLine::Get(), TEXT("-DownloadBuild"), Url))
		{
			FMessageClient MessageClient;
			MessageClient.Start();

			FUrlMessage UrlMessage;
			UrlMessage.Action = EUrlAction::HighlightAndDownload;
			UrlMessage.Url = Url;
			MessageClient.SendSync(UrlMessage);
		}
		else if (FParse::Value(FCommandLine::Get(), TEXT("-GoToBuild"), Url))
		{
			FMessageClient MessageClient;
			MessageClient.Start();

			FUrlMessage UrlMessage;
			UrlMessage.Action = EUrlAction::Highlight;
			UrlMessage.Url = Url;
			MessageClient.SendSync(UrlMessage);
		}
		return true;
	}
	check(GConfig && GConfig->IsReadyForUse());

	// Initialize high DPI mode
	FSlateApplication::InitHighDPI(true);

	FBuildStorageToolParametersBuilder ParametersBuilder;
	FBuildStorageToolParameters Parameters = ParametersBuilder.Build();
	
	{
		// Create the platform slate application (what FSlateApplication::Get() returns)
		TSharedRef<FSlateApplication> Slate = FSlateApplication::Create(MakeShareable(FPlatformApplicationMisc::CreateApplication()));

		{
			// Initialize renderer
			TSharedRef<FSlateRenderer> SlateRenderer = GetStandardStandaloneRenderer();

			// Try to initialize the renderer. It's possible that we launched when the driver crashed so try a few times before giving up.
			bool bRendererInitialized = Slate->InitializeRenderer(SlateRenderer, true);
			if (!bRendererInitialized)
			{
				// Close down the Slate application
				FSlateApplication::Shutdown();
				return false;
			}

			// Set the normal UE IsEngineExitRequested() when outer frame is closed
			Slate->SetExitRequestedHandler(FSimpleDelegate::CreateStatic(&OnRequestExit));

			// Prepare the custom Slate styles
			FBuildStorageToolStyle::Initialize();

			// Set the icon
			FAppStyle::SetAppStyleSet(FBuildStorageToolStyle::Get());

			// Run the inner application loop
			FBuildStorageToolApp App(Slate.Get(), Parameters);
			App.Run();

			// Clean up the custom styles
			FBuildStorageToolStyle::Shutdown();
		}

		// Close down the Slate application
		FSlateApplication::Shutdown();
	}

	return true;
}

#if PLATFORM_WINDOWS
int WINAPI WinMain(_In_ HINSTANCE hCurrInstance, _In_opt_ HINSTANCE hPrevInstance, _In_ LPSTR lpCmdLine, _In_ int nShowCmd)
{
	hInstance = hCurrInstance;
	const FTaskTagScope Scope(ETaskTag::EGameThread);
	return BuildStorageToolMain(GetCommandLineW())? 0 : 1;
}

#elif PLATFORM_LINUX
int UnixMainWrapper(const TCHAR* CommandLine)
{
	const FTaskTagScope Scope(ETaskTag::EGameThread);
	return BuildStorageToolMain(CommandLine) ? 0 : 1;
};
int main(int argc, char* argv[])
{
	return CommonUnixMain(argc, argv, &UnixMainWrapper);
}
#elif PLATFORM_MAC
int32 MacMainWrapper(const TCHAR* CommandLine)
{
	// On Mac the game thread starts with ActiveTaskTag == EStaticInit (thread-local default).
	// FTaskTagScope(EGameThread) constructor would call GetStaticThreadId() from the EStaticInit
	// path, seeding it to the game thread ID. This breaks IsRunningDuringStaticInit() on the
	// main thread during __cxa_finalize_ranges, which needs GetStaticThreadId() to match the
	// main thread. Calling SetTagNone() first skips the EStaticInit path in the constructor,
	// leaving GetStaticThreadId() uninitialized until it is first called from the main thread
	// (in IsRunningDuringStaticInit() during shutdown), where it correctly seeds to main thread ID.
	FTaskTagScope::SetTagNone();
	const FTaskTagScope Scope(ETaskTag::EGameThread);
	return BuildStorageToolMain(CommandLine) ? 0 : 1;
};
int main(int argc, char* argv[])
{
	[MacProgramDelegate mainWithArgc : argc argv : argv programMain : MacMainWrapper programExit : FEngineLoop::AppExit] ;
}
#else
#error "Unsupported platform!"
#endif

#undef LOCTEXT_NAMESPACE
