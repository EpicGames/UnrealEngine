// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildStorageToolHelpWidget.h"

#include "BuildStorageToolStyle.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SBoxPanel.h"

#include "Version/AppVersionDefines.h"

#define LOCTEXT_NAMESPACE "BuildStorageToolHelpWidget"

// TODO BC: data drive all the data for the help page
void SBuildStorageToolHelpWidget::Construct(const FArguments& InArgs)
{
	ToolParameters = InArgs._ToolParameters.Get();

	TSharedPtr<SVerticalBox> Contents;
	ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SAssignNew(Contents, SVerticalBox)
			]
		];

	Contents->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Header", "The Build Storage Tool helps you download builds of different kinds.\n"))
		];

	Contents->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(5)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Documentation", "Please refer to the documentation page(s):"))
		];

	for(const FDocumentationLink& DocLink : ToolParameters->GeneralParameters.HelpLinks)
	{
		Contents->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(10)
			[
				SNew(SHyperlink)
				.Style(FBuildStorageToolStyle::Get(), TEXT("NavigationHyperlink"))
				.Text(FText::FromString(DocLink.Text))
				.ToolTipText(FText::FromString(DocLink.Tooltip))
				.OnNavigate_Lambda([&DocLink]() { FPlatformProcess::LaunchURL(*DocLink.Link, nullptr, nullptr); })
			];
	}

	Contents->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Padding(10)
		[
			SNew(STextBlock)
			.Text(
				FText::Format(
#if defined(BUILD_STORAGE_TOOL_CHANGELIST_STRING)
						LOCTEXT("HelpDialog_VersionFormat", "Application version: {0}"), FText::FromString(BUILD_STORAGE_TOOL_CHANGELIST_STRING)
#else
						LOCTEXT("HelpDialog_UnknownVersion", "Application version: Unknown")
#endif
				)
			)
		];
}

#undef LOCTEXT_NAMESPACE