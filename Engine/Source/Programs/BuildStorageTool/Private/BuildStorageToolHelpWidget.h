// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Parameters/BuildStorageToolParameters.h"

class SBuildStorageToolHelpWidget final : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBuildStorageToolHelpWidget) {}
		SLATE_ATTRIBUTE(const FBuildStorageToolParameters*, ToolParameters)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
private:
	const FBuildStorageToolParameters* ToolParameters;
};