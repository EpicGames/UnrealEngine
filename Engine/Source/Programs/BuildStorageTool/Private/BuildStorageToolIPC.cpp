// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildStorageToolIPC.h"
#include "Algo/RemoveIf.h"
#include "Containers/Ticker.h"
#include "Dom/JsonObject.h"
#include "HAL/PlatformProcess.h"
#include "IPAddress.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryValidation.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Sockets.h"
#include "SocketSubsystem.h"

namespace UE::BuildStorageTool
{
static constexpr uint32 BSTIPC_Magic = 0x42535449; // 'BSTI'
static constexpr uint16 BSTIPC_Version = 1;

#pragma pack(push, 1)
struct FFrameHeader
{
	uint32 Magic;
	uint16 Version;
	uint32 PayloadLen;
};
#pragma pack(pop)

FString GetRendezvousFilePath()
{
	return FPaths::Combine(FPlatformProcess::UserTempDir(), TEXT("BuildStorageTool.ipc.json"));
}

bool WriteRendezvousFileJson(const uint16 Port)
{
	TSharedRef<FJsonObject> RootObject = MakeShared<FJsonObject>();
	RootObject->SetNumberField(TEXT("port"), static_cast<double>(Port));

	FString OutputString;
	const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	if (!FJsonSerializer::Serialize(RootObject, Writer))
	{
		return false;
	}

	// Atomic-ish write: write temp then move over final path.
	const FString FinalPath = GetRendezvousFilePath();
	const FString TempPath = FinalPath + TEXT(".tmp");

	if (!FFileHelper::SaveStringToFile(OutputString, *TempPath))
	{
		return false;
	}

	return IFileManager::Get().Move(*FinalPath, *TempPath, /*bReplace*/ true, /*bEvenIfReadOnly*/ false);
}

bool ReadRendezvousFileJson(uint16& OutPort)
{
	OutPort = 0;

	FString InputString;
	if (!FFileHelper::LoadFileToString(InputString, *GetRendezvousFilePath()))
	{
		return false;
	}

	TSharedPtr<FJsonObject> RootObject;
	const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(InputString);
	if (!FJsonSerializer::Deserialize(Reader, RootObject) || !RootObject.IsValid())
	{
		return false;
	}

	double PortAsNumber = 0.0;
	if (!RootObject->TryGetNumberField(TEXT("port"), PortAsNumber))
	{
		return false;
	}

	const int32 PortAsInt = static_cast<int32>(PortAsNumber);
	if (PortAsInt <= 0 || PortAsInt > 65535)
	{
		return false;
	}

	OutPort = static_cast<uint16>(PortAsInt);
	return true;
}

static FSocket* StartListeningForMessages(uint16& OutPort)
{
	OutPort = 0;

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	if (!SocketSubsystem)
	{
		return nullptr;
	}

	bool bIsValid = false;
	TSharedRef<FInternetAddr> BindAddress = SocketSubsystem->CreateInternetAddr();
	BindAddress->SetIp(TEXT("127.0.0.1"), bIsValid);
	BindAddress->SetPort(0); // Ephemeral port

	if (!bIsValid)
	{
		return nullptr;
	}

	TUniquePtr<FSocket> ListenSocket(
		SocketSubsystem->CreateSocket(NAME_Stream, TEXT("BuildStorageToolIPC"), BindAddress->GetProtocolType())
	);

	if (!ListenSocket.IsValid())
	{
		return nullptr;
	}

	ListenSocket->SetReuseAddr(false);
	ListenSocket->SetNoDelay(true);
	ListenSocket->SetNonBlocking(true);

	const int32 MaxBacklog = 16;
	if (!ListenSocket->Bind(*BindAddress) || !ListenSocket->Listen(MaxBacklog))
	{
		return nullptr;
	}

	TSharedRef<FInternetAddr> BoundAddress = SocketSubsystem->CreateInternetAddr();
	ListenSocket->GetAddress(*BoundAddress);
	OutPort = BoundAddress->GetPort();

	return ListenSocket.Release();
}

static bool RecvAll(FSocket& Socket, void* Data, int32 Length)
{
	uint8* BytePtr = static_cast<uint8*>(Data);

	int32 TotalRead = 0;
	while (TotalRead < Length)
	{
		int32 BytesRead = 0;
		if (!Socket.Recv(BytePtr + TotalRead, Length - TotalRead, BytesRead) || BytesRead <= 0)
		{
			return false;
		}
		TotalRead += BytesRead;
	}

	return true;
}

static bool SendAll(FSocket& Socket, const void* Data, const int32 Length)
{
	const uint8* BytePtr = static_cast<const uint8*>(Data);
	int32 TotalSent = 0;

	while (TotalSent < Length)
	{
		int32 BytesSent = 0;
		if (!Socket.Send(BytePtr + TotalSent, Length - TotalSent, BytesSent) || BytesSent <= 0)
		{
			return false;
		}
		TotalSent += BytesSent;
	}

	return true;
}

void FMessageListenServer::FReceiveBuffer::Reset()
{
	MessageType.Invalidate();
	Payload.Reset();
	bParsedHeader = false;
	BytesRead = 0;
}

FMessageListenServer::FMessageListenServer()
: ShutdownEvent(EEventMode::ManualReset)
{
}

bool FMessageListenServer::Start()
{
	uint16 ListenPort = 0;
	ListenSocket = StartListeningForMessages(ListenPort);
	if (ListenSocket)
	{
		if (ListenPort != 0)
		{
			bWroteRendezvousFile = WriteRendezvousFileJson(ListenPort);
		}

		WorkerThread = FThread(TEXT("BuildStorageToolIPC_Thread"), [this] { ThreadLoop(); }, 128 * 1024);
		return true;
	}
	return false;
}

void FMessageListenServer::ShutdownJoin()
{
	ShutdownEvent->Trigger();
	if (WorkerThread.IsJoinable())
	{
		WorkerThread.Join();
	}
	ShutdownEvent->Reset();
}

void FMessageListenServer::ThreadLoop()
{
	constexpr float TickPeriod = 1.f;
	constexpr float MinSleepTime = 0.001f;
	for (;;)
	{
		double StartTime = FPlatformTime::Seconds();

		bool bReadReady;
		while (ListenSocket->HasPendingConnection(bReadReady) && bReadReady)
		{
			if (FSocket* ClientSocket = ListenSocket->Accept(TEXT("Client Connection")))
			{
				ClientSocket->SetNonBlocking(true);
				ClientStates.AddDefaulted_GetRef().Socket = ClientSocket;
			}
		}

		TickCommunication();

		double CurrentTime = FPlatformTime::Seconds();
		float RemainingDuration = StartTime + TickPeriod - CurrentTime;
		if (RemainingDuration > .001f)
		{
			uint32 WaitTimeMilliseconds = static_cast<uint32>(RemainingDuration * 1000);
			if (ShutdownEvent->Wait(WaitTimeMilliseconds))
			{
				break;
			}
		}
		else if (ShutdownEvent->Wait(0))
		{
			break;
		}
	}

	// Cleanup
	if (ListenSocket)
	{
		ListenSocket->Close();
		ISocketSubsystem::Get()->DestroySocket(ListenSocket);
		ListenSocket = nullptr;
	}
	for (FClientState& ClientState : ClientStates)
	{
		if (ClientState.Socket)
		{
			ClientState.Socket->Close();
			ISocketSubsystem::Get()->DestroySocket(ClientState.Socket);
			ClientState.Socket = nullptr;
		}
	}
	ClientStates.Empty();
}

void FMessageListenServer::TickCommunication()
{
	for (FClientState& ClientState : ClientStates)
	{
		if (!ClientState.Socket)
		{
			continue;
		}

		TArray<FMarshalledMessage> Messages;
		constexpr uint64 MaxPayloadLen = 16 * 1024 * 1024;
		EConnectionStatus SocketStatus = TryReadPacket(*ClientState.Socket, ClientState.ReceiveBuffer, Messages, MaxPayloadLen);
		if (SocketStatus != EConnectionStatus::Okay && SocketStatus != EConnectionStatus::Incomplete)
		{
			ClientState.Socket->Close();
			ISocketSubsystem::Get()->DestroySocket(ClientState.Socket);
			ClientState.Socket = nullptr;
		}

		for (const FMarshalledMessage& Message : Messages)
		{
			if (Message.MessageType == FUrlMessage::MessageType)
			{
				FUrlMessage UrlMessage;
				if (UrlMessage.TryRead(Message.Object))
				{
					ExecuteOnGameThread(UE_SOURCE_LOCATION,
						[this, UrlMessage = MoveTemp(UrlMessage)]
						{

							if (OnMessageReceived.IsBound())
							{
								OnMessageReceived.Broadcast(UrlMessage);
							}
						});
				}
			}
		}
	}

	ClientStates.SetNum(Algo::RemoveIf(ClientStates, [] (FClientState& ClientState)
	{
		return ClientState.Socket == nullptr;
	}));
}

FMessageListenServer::EConnectionStatus FMessageListenServer::TryReadPacket(FSocket& Socket, FReceiveBuffer& Buffer,
	TArray<FMarshalledMessage>& Messages, uint64 MaxPayloadLen)
{
	int32 BytesRead;
	if (!Buffer.bParsedHeader)
	{
		// When reading the relatively small FrameHeader, we wait for its total size to be ready
		// using Peek to reduce complexity in this function.
		FFrameHeader MessageHeader;
		bool bStillAlive = Socket.Recv(reinterpret_cast<uint8*>(&MessageHeader), sizeof(MessageHeader),
			BytesRead, ESocketReceiveFlags::Peek);
		check(BytesRead <= sizeof(FFrameHeader));
		if (BytesRead < sizeof(FFrameHeader))
		{
			if (!bStillAlive)
			{
				return EConnectionStatus::Terminated;
			}
			else
			{
				return EConnectionStatus::Okay;
			}
		}

		Socket.Recv(reinterpret_cast<uint8*>(&MessageHeader), sizeof(MessageHeader), BytesRead,
			ESocketReceiveFlags::None);
		check(BytesRead == sizeof(FFrameHeader));
		if (MessageHeader.Magic != BSTIPC_Magic || MessageHeader.Version != BSTIPC_Version ||
			(MaxPayloadLen > 0 && MessageHeader.PayloadLen > MaxPayloadLen))
		{
			return EConnectionStatus::FormatError;
		}

		Buffer.Payload = FUniqueBuffer::Alloc(MessageHeader.PayloadLen);
		Buffer.bParsedHeader = true;
		Buffer.BytesRead = 0;
	}


	while (Buffer.Payload.GetSize() > Buffer.BytesRead)
	{
		// When reading the possibly large payload size that will block the remote socket
		// from continuing until we have read some of it, we read as much of the payload
		// as is available and store it in our dynamic and larger buffer in Buffer.Payload.
		uint64 RemainingSize = Buffer.Payload.GetSize() - Buffer.BytesRead;
		// Avoid possible OS restrictions on maximum read size by imposing our own moderate
		// size per call to Recv.
		constexpr uint64 MaxReadSize = 1000 * 1000 * 64;
		int32 ReadSize = static_cast<int32>(FMath::Min(RemainingSize, MaxReadSize));
		uint8* ReadData = static_cast<uint8*>(Buffer.Payload.GetData()) + Buffer.BytesRead;
		bool bConnectionAlive = Socket.Recv(ReadData, ReadSize, BytesRead);
		if (BytesRead <= 0)
		{
			if (!bConnectionAlive)
			{
				return EConnectionStatus::Terminated;
			}
			else
			{
				return EConnectionStatus::Okay;
			}
		}

		check(BytesRead <= ReadSize);
		Buffer.BytesRead += BytesRead;
	}

	// The FCbObjects we return have a pointer to this buffer and will keep it allocated
	// until they are destructed.
	FSharedBuffer SharedBuffer = Buffer.Payload.MoveToShared();
	Buffer.Reset();
	if (ValidateCompactBinary(SharedBuffer, ECbValidateMode::Default) != ECbValidateError::None)
	{
		return EConnectionStatus::FormatError;
	}

	FCbFieldIterator It = FCbFieldIterator::MakeRange(SharedBuffer);
	while (It)
	{
		FMarshalledMessage& Message = Messages.Emplace_GetRef();
		FCbFieldView MessageTypeView = *It;
		Message.MessageType = MessageTypeView.AsUuid();
		if (MessageTypeView.HasError())
		{
			return EConnectionStatus::FormatError;
		}
		++It;
		FCbField ObjectField = *It;
		Message.Object = ObjectField.AsObject();
		if (ObjectField.HasError())
		{
			return EConnectionStatus::FormatError;
		}
		++It;
	}
	return EConnectionStatus::Okay;
}

FMessageClient::FMessageClient()
: ShutdownEvent(EEventMode::ManualReset)
{
}

bool FMessageClient::Start()
{
	if (!ReadRendezvousFileJson(Port))
	{
		return false;
	}

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	if (!SocketSubsystem)
	{
		return false;
	}

	bool bIsValid = false;
	TSharedRef<FInternetAddr> RemoteAddress = SocketSubsystem->CreateInternetAddr();
	RemoteAddress->SetIp(TEXT("127.0.0.1"), bIsValid);
	RemoteAddress->SetPort(Port);

	if (!bIsValid)
	{
		return false;
	}

	Socket = TUniquePtr<FSocket>(SocketSubsystem->CreateSocket(NAME_Stream, TEXT("BuildStorageToolIPC_Send"), RemoteAddress->GetProtocolType()));

	if (!Socket.IsValid())
	{
		return false;
	}

	// Small retry window for early startup race
	const double StartTime = FPlatformTime::Seconds();
	while (!Socket->Connect(*RemoteAddress))
	{
		if (FPlatformTime::Seconds() - StartTime > 3.0)
		{
			return false;
		}
		FPlatformProcess::Sleep(0.02f);
	}

	bReady = true;
	return true;
}

bool FMessageClient::SendSync(const IMessage& Message)
{
	if (!bReady || !Socket)
	{
		return false;
	}

	FFrameHeader Header;
	Header.Magic = BSTIPC_Magic;
	Header.Version = BSTIPC_Version;
	FCbWriter Writer;
	Writer.AddUuid(Message.GetMessageType());
	Writer.BeginObject();
	Message.Write(Writer);
	Writer.EndObject();
	uint64 PayloadLen64 = Writer.GetSaveSize();

	if (PayloadLen64 > (MAX_int32 - sizeof(Header)))
	{
		return false;
	}

	Header.PayloadLen = static_cast<uint32>(PayloadLen64);

	FUniqueBuffer SendBuffer = FUniqueBuffer::Alloc(sizeof(Header) + Header.PayloadLen);
	FMemory::Memcpy(SendBuffer.GetData(), &Header, sizeof(Header));
	FMutableMemoryView PayloadView(SendBuffer.GetView().RightChop(sizeof(Header)));
	Writer.Save(PayloadView);

	return SendAll(*Socket, SendBuffer.GetData(), static_cast<int32>(SendBuffer.GetSize()));
}

template <typename CharType>
static TStringBuilderBase<CharType>& UrlActionToString(TStringBuilderBase<CharType>& Builder, const EUrlAction Action)
{
	switch (Action)
	{
	case EUrlAction::Highlight:               return Builder << ANSITEXTVIEW("Highlight");
	case EUrlAction::HighlightAndDownload:    return Builder << ANSITEXTVIEW("HighlightAndDownload");
	}
	return Builder << ANSITEXTVIEW("Unknown");
}

template <typename CharType>
static bool UrlActionFromString(EUrlAction& OutAction, const TStringView<CharType> String)
{
	const auto ConvertedString = StringCast<UTF8CHAR, 16>(String.GetData(), String.Len());
	if (ConvertedString == UTF8TEXTVIEW("Highlight"))
	{
		OutAction = EUrlAction::Highlight;
	}
	else if (ConvertedString == UTF8TEXTVIEW("HighlightAndDownload"))
	{
		OutAction = EUrlAction::HighlightAndDownload;
	}
	else
	{
		return false;
	}
	return true;
}

FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, EUrlAction Action) { return UrlActionToString(Builder, Action); }
FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, EUrlAction Action) { return UrlActionToString(Builder, Action); }
FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, EUrlAction Action) { return UrlActionToString(Builder, Action); }

bool TryLexFromString(EUrlAction& OutAction, FUtf8StringView String) { return UrlActionFromString(OutAction, String); }
bool TryLexFromString(EUrlAction& OutAction, FWideStringView String) { return UrlActionFromString(OutAction, String); }

FCbWriter& operator<<(FCbWriter& Writer, const EUrlAction Action)
{
	Writer.AddString(WriteToUtf8String<16>(Action));
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, EUrlAction& OutAction)
{
	if (TryLexFromString(OutAction, Field.AsString()))
	{
		return true;
	}
	OutAction = {};
	return false;
}

FGuid FUrlMessage::MessageType(TEXT("FC6BC9307BAB4D4C9F0C236FDEF2BE62"));

FUrlMessage::FUrlMessage()
{
}

void FUrlMessage::Write(FCbWriter& Writer) const
{
	Writer << "A" << Action;
	Writer << "U" << Url;
}

bool FUrlMessage::TryRead(FCbObjectView Object)
{
	bool bOk = true;
	bOk = LoadFromCompactBinary(Object["A"], Action) & bOk;
	bOk = LoadFromCompactBinary(Object["U"], Url) & bOk;
	return bOk;
}

} // namespace UE::BuildStorageTool
