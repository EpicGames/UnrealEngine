// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Event.h"
#include "HAL/Thread.h"
#include "Misc/Guid.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Sockets.h"
#include "Templates/UniquePtr.h"

namespace UE::BuildStorageTool
{

FString GetRendezvousFilePath();
bool WriteRendezvousFileJson(const uint16 Port);
bool ReadRendezvousFileJson(uint16& OutPort);

class IMessage
{
public:
	virtual ~IMessage() = default;

	/** Marshall the message to a CompactBinaryObject. */
	virtual void Write(FCbWriter& Writer) const = 0;
	/** Unmarshall the message from a CompactBinaryObject. */
	virtual bool TryRead(FCbObjectView Object) = 0;
	/** Return the Guid that identifies the message to the remote connection. */
	virtual FGuid GetMessageType() const = 0;
	/** Return the debugname for diagnostics. */
	virtual const TCHAR* GetDebugName() const = 0;
};

typedef TMulticastDelegate<void(const IMessage& )> FMessageReceivedDelegate;

class FMessageListenServer
{
public:
	FMessageListenServer();

	bool Start();
	void ShutdownJoin();

	FMessageReceivedDelegate OnMessageReceived;
private:
	enum class EConnectionStatus
	{
		/** Connection is still okay or operation succeeded. */
		Okay,
		/** Connection failed and no further use of the Socket is possible. */
		Terminated,
		/** The Socket is still active but the data received was invalid and recovery is not possible. */
		FormatError,
		/** The operation failed and the Socket is no longer usable. */
		Failed,
		/** The Socket is busy and the operation needs to be retried later */
		Incomplete,
	};

	struct FReceiveBuffer
	{
	public:
		void Reset();

		FGuid MessageType;
		FUniqueBuffer Payload;
		uint64 BytesRead = 0;
		bool bParsedHeader = false;
	};

	struct FMarshalledMessage
	{
		FGuid MessageType;
		FCbObject Object;
	};

	void ThreadLoop();
	void TickCommunication();
	EConnectionStatus TryReadPacket(FSocket& Socket, FReceiveBuffer& Buffer,
	TArray<FMarshalledMessage>& Messages, uint64 MaxPayloadLen);

	struct FClientState
	{
		FSocket* Socket = nullptr;
		FReceiveBuffer ReceiveBuffer;
	};
	TArray<FClientState> ClientStates;
	FEventRef ShutdownEvent;
	FThread WorkerThread;
	FSocket* ListenSocket = nullptr;
	uint16 Port = 0;
	bool bWroteRendezvousFile = false;
};

class FMessageClient
{
public:
	FMessageClient();

	bool Start();
	bool SendSync(const IMessage& Message);

private:
	void ThreadLoop();

	FEventRef ShutdownEvent;
	FThread WorkerThread;
	TUniquePtr<FSocket> Socket;
	uint16 Port = 0;
	bool bReady = false;
};

enum class EUrlAction : uint8
{
	Highlight,
	HighlightAndDownload,
};

FAnsiStringBuilderBase& operator<<(FAnsiStringBuilderBase& Builder, EUrlAction Action);
FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, EUrlAction Action);
FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, EUrlAction Action);

bool TryLexFromString(EUrlAction& OutAction, FUtf8StringView String);
bool TryLexFromString(EUrlAction& OutAction, FWideStringView String);

struct FUrlMessage : public IMessage
{
public:
	FUrlMessage();
	virtual void Write(FCbWriter& Writer) const override;
	virtual bool TryRead(FCbObjectView Object) override;
	virtual FGuid GetMessageType() const override { return MessageType; }
	virtual const TCHAR* GetDebugName() const override { return TEXT("URLMessage"); }

public:
	EUrlAction Action;
	FString Url;
	static FGuid MessageType;
};

} // namespace UE::BuildStorageTool
