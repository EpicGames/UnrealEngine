// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildStorageToolStyle.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Misc/Paths.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"

#define EDITOR_IMAGE_BRUSH(RelativePath, ...) IMAGE_BRUSH("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_IMAGE_BRUSH_SVG(RelativePath, ...) IMAGE_BRUSH_SVG("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BOX_BRUSH(RelativePath, ...) BOX_BRUSH("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define EDITOR_BORDER_BRUSH(RelativePath, ...) BORDER_BRUSH("../Editor/Slate/" RelativePath, __VA_ARGS__)
#define RootToContentDir ContentFromEngine
#define RootToCoreContentDir ContentFromEngine

TSharedPtr< FSlateStyleSet > FBuildStorageToolStyle::StyleSet = nullptr;

void FBuildStorageToolStyle::Initialize()
{
	if(!StyleSet.IsValid())
	{
		StyleSet = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
	}
}

void FBuildStorageToolStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet);
	ensure(StyleSet.IsUnique());
	StyleSet.Reset();
}

namespace
{
	FString ContentFromEngine(const FString& RelativePath, const TCHAR* Extension)
	{
		static const FString ContentDir = FPaths::EngineDir() / TEXT("Content/Slate");
		return ContentDir / RelativePath + Extension;
	}
}

TSharedRef< FSlateStyleSet > FBuildStorageToolStyle::Create()
{
	TSharedRef<FSlateStyleSet> StyleRef = MakeShareable(new FSlateStyleSet("BuildStorageToolStyle"));
	FSlateStyleSet& Style = StyleRef.Get();

	Style.SetParentStyleName("CoreStyle");

	const ISlateStyle* ParentStyle = FSlateStyleRegistry::FindSlateStyle("CoreStyle");
	const FTextBlockStyle NormalText = ParentStyle->GetWidgetStyle<FTextBlockStyle>("NormalText");

	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);

	// {
	// 	const FString IconPath = FPaths::EngineDir() / TEXT("Programs/BuildStorageTool/Resources/Windows/icon_24x24.png");
	// 	Style.Set("AppIcon", new FSlateImageBrush(IconPath, FVector2D(24, 24)));
	// 	Style.Set("AppIcon.Small", new FSlateImageBrush(IconPath, FVector2D(24, 24)));
	// 	Style.Set("AppIconPadding", FMargin(5.f, 5.f, 5.f, 5.f));
	// 	Style.Set("AppIconPadding.Small", FMargin(4.f, 4.f, 4.f, 4.f));
	// }

	Style.Set("Zen.FolderExplore", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", Icon12x12));
	Style.Set("Zen.BrowseContent", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/BrowseContent", Icon16x16));
	Style.Set("Zen.Clipboard", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/Clipboard", Icon16x16));
	Style.Set("Zen.Filter", new CORE_IMAGE_BRUSH_SVG("Starship/Common/filter", Icon16x16));

	Style.Set("Zen.FolderView", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon12x12));
	Style.Set("Zen.BrowserView", new EDITOR_IMAGE_BRUSH_SVG("Starship/Common/WebBrowser", Icon12x12));

	Style.Set("Zen.Cancel", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reject", Icon12x12));

	const FButtonStyle NoPaddingSimpleButton = FButtonStyle()
	.SetNormal(FSlateNoResource())
	.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
	.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
	.SetDisabled(FSlateNoResource())
	.SetNormalForeground(FStyleColors::Foreground)
	.SetHoveredForeground(FStyleColors::ForegroundHover)
	.SetPressedForeground(FStyleColors::Primary)
	.SetDisabledForeground(FStyleColors::Foreground)
	.SetNormalPadding(0)
	.SetPressedPadding(0);
	FComboButtonStyle SmallContextCombo = FComboButtonStyle(Style.GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		.SetContentPadding(FMargin(3.f, 0.f))
		.SetButtonStyle(NoPaddingSimpleButton)
		.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(4.f, 16.f)));

	Style.Set("Artifact.SmallContext", SmallContextCombo);

	// FlatButton styles — square-cornered buttons matching the editor's StarshipStyle definitions.
	// Not inherited from CoreStyle so must be defined here for the standalone tool.
	{
		const FButtonStyle& BaseButton = FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
		const FLinearColor SelectionColor = FStyleColors::Primary.GetSpecifiedColor();
		const FLinearColor SelectionColorPressed = SelectionColor * 0.75f;

		Style.Set("FlatButton", FButtonStyle(BaseButton)
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColorPressed))
		);

		const FLinearColor DarkNormal(0.125f, 0.125f, 0.125f, 0.8f);

		Style.Set("FlatButton.Dark", FButtonStyle(BaseButton)
			.SetNormal(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, DarkNormal))
			.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColorPressed))
		);

		Style.Set("FlatButton.Default", Style.GetWidgetStyle<FButtonStyle>("FlatButton.Dark"));
	}

	const FTextBlockStyle DefaultText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Bold", 10))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black);

	// Set the client app styles
	Style.Set(TEXT("Code"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Regular", 8))
		.SetColorAndOpacity(FSlateColor(FLinearColor::White * 0.8f))
	);

	Style.Set(TEXT("Title"), FTextBlockStyle(DefaultText)
		.SetFont(DEFAULT_FONT("Bold", 12))
	);

	Style.Set(TEXT("Status"), FTextBlockStyle(DefaultText)
		.SetColorAndOpacity(FSlateColor::UseSubduedForeground())
	);

	{
		// Navigation defaults
		const FLinearColor NavHyperlinkColor(0.03847f, 0.33446f, 1.0f);
		const FTextBlockStyle NavigationHyperlinkText = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(NavHyperlinkColor);

		const FButtonStyle NavigationHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor));

		FHyperlinkStyle NavigationHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(NavigationHyperlinkButton)
			.SetTextStyle(NavigationHyperlinkText)
			.SetPadding(FMargin(0.0f));

		Style.Set("NavigationHyperlink", NavigationHyperlink);
	}


	FSlateBrush* GenericWhiteBox = new IMAGE_BRUSH( "Old/White", Icon16x16 );

	// Scrollbar
	const FScrollBarStyle ScrollBar = FScrollBarStyle()
		.SetVerticalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetVerticalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetHorizontalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetHorizontalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetNormalThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetDraggedThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)))
		.SetHoveredThumbImage(BOX_BRUSH("Common/Scrollbar_Thumb", FMargin(4.f / 16.f)));

	Style.Set("Log.TextBox", FEditableTextBoxStyle(FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
		.SetBackgroundImageNormal( BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f/16.0f)))
		.SetBackgroundImageHovered( BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f/16.0f)))
		.SetBackgroundImageFocused(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageReadOnly(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
		.SetBackgroundColor( FLinearColor(0.015f, 0.015f, 0.015f) )
		.SetScrollBarStyle(ScrollBar)
		);

	return StyleRef;
}

const ISlateStyle& FBuildStorageToolStyle::Get()
{
	return *StyleSet;
}

#undef EDITOR_IMAGE_BRUSH
#undef EDITOR_IMAGE_BRUSH_SVG
#undef EDITOR_BOX_BRUSH
#undef EDITOR_BORDER_BRUSH
#undef RootToContentDir
