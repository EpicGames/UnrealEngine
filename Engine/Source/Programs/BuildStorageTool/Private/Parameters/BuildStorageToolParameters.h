// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Class.h"
#include "BuildStorageToolParameters.generated.h"

USTRUCT()
struct FDocumentationLink
{
	GENERATED_BODY()

	UPROPERTY()
	FString Text;
	
	UPROPERTY()
	FString Tooltip;

	UPROPERTY()
	FString Link;
};

USTRUCT()
struct FGeneralParameters 
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FDocumentationLink> HelpLinks;
};

struct FBuildStorageToolParameters
{
	FGeneralParameters GeneralParameters;
};
