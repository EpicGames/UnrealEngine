// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildStorageToolParametersBuilder.h"

#include "CoreGlobals.h"
#include "BuildStorageTool.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigContext.h"
#include "Misc/StringOutputDevice.h"

FBuildStorageToolParametersBuilder::FBuildStorageToolParametersBuilder()
{
	
}

FBuildStorageToolParameters FBuildStorageToolParametersBuilder::Build()
{	
	FString IniFilename = TEXT("Engine");
	BuildStorageToolConfig = GConfig->FindConfigFile(IniFilename);

	FBuildStorageToolParameters Parameters;
	Parameters.GeneralParameters = BuildGeneralParameters();
	return Parameters;
}

FGeneralParameters FBuildStorageToolParametersBuilder::BuildGeneralParameters()
{
	const FConfigSection* Section = BuildStorageToolConfig->FindSection(TEXT("BuildStorageTool.General"));
	FGeneralParameters Output;

	if(Section != nullptr)
	{
		FStringOutputDevice Errors;
		FGeneralParameters::StaticStruct()->ImportText(*SectionToText(*Section), &Output, nullptr, 0, &Errors, FGeneralParameters::StaticStruct()->GetName());

		if(!Errors.IsEmpty())
		{
			// TODO: Do we want to do more then just log a error here? 
			UE_LOGF(LogBuildStorageTool, Error, "Error loading parameter file %ls", *Errors);
		}
	}

	return Output;
}

FString FBuildStorageToolParametersBuilder::SectionToText(const FConfigSection& InSection) const
{
	TArray<FString> lines;
	for(const TPair<FName, FConfigValue>& Item : InSection.Array())
	{
		FString Value = Item.Value.GetValue();

		// If it's an array/map/struct, we only need to quote the key, otherwise quote key and value
		if((Value.IsNumeric() && !Value.Equals(TEXT("-"))) || (Value.StartsWith(TEXT("(")) && Value.EndsWith(TEXT(")")) && !Item.Key.ToString().Contains(TEXT("Regex"), ESearchCase::IgnoreCase)))
		{
			lines.Add(FString::Printf(TEXT("\"%s\"=%s"), *Item.Key.ToString(), *Item.Value.GetValue()));
		}
		else
		{
			lines.Add(FString::Printf(TEXT("\"%s\"=\"%s\""), *Item.Key.ToString(), *Item.Value.GetValue()));
		}
	}

	FString FinalText = TEXT("(") + FString::Join(lines, TEXT(",")) + TEXT(")");
	return FinalText;
}

