// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Misc/ConfigCacheIni.h"
#include "BuildStorageToolParameters.h"

class FBuildStorageToolParametersBuilder
{
public:
	FBuildStorageToolParametersBuilder();
	FBuildStorageToolParameters Build();

private:
	FConfigFile* BuildStorageToolConfig;
private:
	FGeneralParameters BuildGeneralParameters();
	
	FString SectionToText(const FConfigSection& InSection) const;
};
