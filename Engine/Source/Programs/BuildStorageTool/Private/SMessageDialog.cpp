// Copyright Epic Games, Inc. All Rights Reserved.


#include "SMessageDialog.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Misc/Attribute.h"
#include "SlateOptMacros.h"
#include "Styling/CoreStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "Dialogs"

class SChoiceDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SChoiceDialog )	{}
		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(FText, Message)	
		SLATE_ATTRIBUTE(float, WrapMessageAt)
		SLATE_ARGUMENT(EAppMsgCategory, MessageCategory)
		SLATE_ARGUMENT(EAppMsgType::Type, MessageType)
	SLATE_END_ARGS()

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct( const FArguments& InArgs )
	{
		ParentWindow = InArgs._ParentWindow;
		ParentWindow->SetWidgetToFocusOnActivate(SharedThis(this));
		Response = EAppReturnType::Cancel;
		MessageType = InArgs._MessageType;

		FSlateFontInfo MessageFont( FAppStyle::GetFontStyle("StandardDialog.LargeFont"));
		MyMessage = InArgs._Message;

		TSharedPtr<SUniformGridPanel> ButtonBox;

		const FSlateBrush* IconBrush = FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large");
		switch (InArgs._MessageCategory)
		{
		case EAppMsgCategory::Error:
			IconBrush = FAppStyle::Get().GetBrush("Icons.ErrorWithColor.Large");
			break;
		case EAppMsgCategory::Success:
			IconBrush = FAppStyle::Get().GetBrush("Icons.SuccessWithColor.Large");
			break;
		case EAppMsgCategory::Info:
			IconBrush = FAppStyle::Get().GetBrush("Icons.InfoWithColor.Large");
			break;
		default:
			break;
		}

		this->ChildSlot
		[	
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(16.f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillHeight(1.0f)
				.MaxHeight(550)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(24.f, 24.f))
						.Image(IconBrush)
					]

					+SHorizontalBox::Slot()
					.Padding(16.f, 0.f, 0.f, 0.f)
					[
						SNew(SScrollBox)
						+ SScrollBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(MyMessage)
							.Font(MessageFont)
							.WrapTextAt(InArgs._WrapMessageAt)
						]
					]
				]

				+SVerticalBox::Slot()
				.Padding(FMargin(0.f, 32.f, 0.f, 0.f))
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.OnClicked(this, &SChoiceDialog::HandleCopyMessageButtonClicked)
						.ToolTipText(NSLOCTEXT("SChoiceDialog", "CopyMessageTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
						.ContentPadding(2.f)
						.Content()
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Zen.Clipboard"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]

					+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(FMargin(16.f, 0.f, 0.f, 0.f))
						[
							SNew(SCheckBox)
							.IsChecked(ECheckBoxState::Unchecked)
							.OnCheckStateChanged(this, &SChoiceDialog::OnCheckboxClicked)
							.Visibility(this, &SChoiceDialog::GetCheckboxVisibility)
							.ToolTipText(NSLOCTEXT("SChoiceDialog", "ApplyToAllTooltip", "Make your choice of Yes or No apply to all remaining items in the current operation"))
							[
								SNew(STextBlock)
								.WrapTextAt(615.0f)
								.Text(NSLOCTEXT("SChoiceDialog", "ApplyToAllLabel", "Apply to All"))
							]
						]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						SAssignNew( ButtonBox, SUniformGridPanel )
						.SlotPadding(FMargin(16.f, 0.f, 0.f, 0.f))
						.MinDesiredSlotWidth(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotWidth"))
						.MinDesiredSlotHeight(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotHeight"))
					]
				]
			]
		];

		int32 SlotIndex = 0;

#define ADD_SLOT(Button)\
		ButtonBox->AddSlot(SlotIndex++,0)\
		[\
			SNew( SButton )\
			.VAlign(VAlign_Center)\
			.HAlign(HAlign_Center)\
			.Text( EAppReturnTypeToText(EAppReturnType::Button) )\
			.OnClicked( this, &SChoiceDialog::HandleButtonClicked, EAppReturnType::Button )\
		];

#define ADD_SLOT_PRIMARY(Button)\
		ButtonBox->AddSlot(SlotIndex++,0)\
		[\
			SNew( SButton )\
			.VAlign(VAlign_Center)\
			.HAlign(HAlign_Center)\
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))\
			.Text( EAppReturnTypeToText(EAppReturnType::Button) )\
			.OnClicked( this, &SChoiceDialog::HandleButtonClicked, EAppReturnType::Button )\
		];



		switch ( MessageType )
		{	
		case EAppMsgType::Ok:
			ADD_SLOT_PRIMARY(Ok)
			break;
		case EAppMsgType::YesNo:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			break;
		case EAppMsgType::OkCancel:
			ADD_SLOT_PRIMARY(Ok)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::YesNoCancel:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::CancelRetryContinue:
			ADD_SLOT_PRIMARY(Continue)
			ADD_SLOT(Retry)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::YesNoYesAllNoAll:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			break;
		case EAppMsgType::YesNoYesAllNoAllCancel:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			ADD_SLOT(Cancel)
			break;
		case EAppMsgType::YesNoYesAll:
			ADD_SLOT_PRIMARY(Yes)
			ADD_SLOT(No)
			ADD_SLOT(YesAll)
			break;
		}

#undef ADD_SLOT
#undef ADD_SLOT_PRIMARY
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION


	EAppReturnType::Type GetResponse()
	{
		return Response;
	}

	virtual	FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		if( InKeyEvent.GetKey() == EKeys::Escape )
		{
			return HandleButtonClicked(EAppReturnType::Cancel);
		}

		if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
		{
			CopyMessageToClipboard();

			return FReply::Handled();
		}

		//if it was some other button, ignore it
		return FReply::Unhandled();
	}

	/** Override the base method to allow for keyboard focus */
	virtual bool SupportsKeyboardFocus() const
	{
		return true;
	}

	/** Converts an EAppReturnType into a localized FText */
	static FText EAppReturnTypeToText(EAppReturnType::Type ReturnType)
	{
		switch(ReturnType)
		{
		case EAppReturnType::No:
			return LOCTEXT("EAppReturnTypeNo", "No");
		case EAppReturnType::Yes:
			return LOCTEXT("EAppReturnTypeYes", "Yes");
		case EAppReturnType::YesAll:
			return LOCTEXT("EAppReturnTypeYesAll", "Yes All");
		case EAppReturnType::NoAll:
			return LOCTEXT("EAppReturnTypeNoAll", "No All");
		case EAppReturnType::Cancel:
			return LOCTEXT("EAppReturnTypeCancel", "Cancel");
		case EAppReturnType::Ok:
			return LOCTEXT("EAppReturnTypeOk", "OK");
		case EAppReturnType::Retry:
			return LOCTEXT("EAppReturnTypeRetry", "Retry");
		case EAppReturnType::Continue:
			return LOCTEXT("EAppReturnTypeContinue", "Continue");
		default:
			return LOCTEXT("MissingType", "MISSING RETURN TYPE");
		}
	}

protected:

	/**
	 * Copies the message text to the clipboard.
	 */
	void CopyMessageToClipboard( )
	{
		FPlatformApplicationMisc::ClipboardCopy( *MyMessage.Get().ToString() );
	}


private:

	// Handles clicking a message box button.
	FReply HandleButtonClicked( EAppReturnType::Type InResponse )
	{
		Response = InResponse;
		if ((MessageType == EAppMsgType::YesNoYesAllNoAll || MessageType == EAppMsgType::YesNoYesAllNoAllCancel)
			&& bApplyToAllChecked)
		{
			if (Response == EAppReturnType::Yes)
			{
				Response = EAppReturnType::YesAll;
			}
			else if (Response == EAppReturnType::No)
			{
				Response = EAppReturnType::NoAll;
			}
		}

		ResultCallback.ExecuteIfBound(ParentWindow.ToSharedRef(), Response);


		ParentWindow->RequestDestroyWindow();

		return FReply::Handled();
	}

	// Handles clicking the 'Copy Message' button.
	FReply HandleCopyMessageButtonClicked()
	{
		CopyMessageToClipboard();
		return FReply::Handled();
	}
		
	// Used as a delegate for the OnClicked property of the Apply to All checkbox
	void OnCheckboxClicked(ECheckBoxState InNewState)
	{
		bApplyToAllChecked = InNewState == ECheckBoxState::Checked;
	}

	// Used as a delegate for the Visibility property of the Apply to All checkbox
	EVisibility GetCheckboxVisibility() const
	{
		if (MessageType == EAppMsgType::YesNoYesAllNoAll || MessageType == EAppMsgType::YesNoYesAllNoAllCancel)
		{
			return EVisibility::Visible;
		}
		return EVisibility::Hidden;
	}

public:
	/** Callback delegate that is triggered, when the dialog is run in non-modal mode */
	FOnMsgDlgResult ResultCallback;

private:

	EAppReturnType::Type Response;
	TSharedPtr<SWindow> ParentWindow;
	TAttribute<FText> MyMessage;
	bool bApplyToAllChecked;
	EAppMsgType::Type MessageType;
};


void CreateMsgDlgWindow(TSharedPtr<SWindow>& OutWindow, TSharedPtr<SChoiceDialog>& OutDialog, EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType,
						const FText& InMessage, const FText& InTitle, FOnMsgDlgResult ResultCallback=NULL)
{
	OutWindow = SNew(SWindow)
		.Title(InTitle)
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PreferredWorkArea)
		.SupportsMinimize(false).SupportsMaximize(false);

	OutDialog = SNew(SChoiceDialog)
		.ParentWindow(OutWindow)
		.Message(InMessage)
		.WrapMessageAt(512.0f)
		.MessageCategory(InMessageCategory)
		.MessageType(InMessageType);

	OutDialog->ResultCallback = ResultCallback;

	OutWindow->SetContent(OutDialog.ToSharedRef());
}

EAppReturnType::Type OpenModalMessageDialog_Internal(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType, EAppReturnType::Type InDefaultValue, const FText& InMessage, const FText& InTitle, const TSharedPtr<const SWidget>& ModalParent)
{
	EAppReturnType::Type Response = InDefaultValue;
	TSharedPtr<SWindow> MsgWindow = NULL;
	TSharedPtr<SChoiceDialog> MsgDialog = NULL;

	CreateMsgDlgWindow(MsgWindow, MsgDialog, InMessageCategory, InMessageType, InMessage, InTitle);

	FSlateApplication::Get().AddModalWindow(MsgWindow.ToSharedRef(), ModalParent);

	Response = MsgDialog->GetResponse();

	return Response;
}

EAppReturnType::Type OpenModalMessageDialog_Internal(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle, const TSharedPtr<const SWidget>& ModalParent)
{
	EAppReturnType::Type DefaultValue = EAppReturnType::Yes;
	switch (InMessageType)
	{
	case EAppMsgType::Ok:
		DefaultValue = EAppReturnType::Ok;
		break;
	case EAppMsgType::YesNo:
		DefaultValue = EAppReturnType::No;
		break;
	case EAppMsgType::OkCancel:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoCancel:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::CancelRetryContinue:
		DefaultValue = EAppReturnType::Cancel;
		break;
	case EAppMsgType::YesNoYesAllNoAll:
		DefaultValue = EAppReturnType::No;
		break;
	case EAppMsgType::YesNoYesAllNoAllCancel:
		DefaultValue = EAppReturnType::Cancel;
		break;
	default:
		DefaultValue = EAppReturnType::Yes;
		break;
	}

	if (GIsRunningUnattendedScript && InMessageType != EAppMsgType::Ok)
	{
		//UE_LOGF(LogDialogs, Error, "Message Dialog was triggered in unattended script mode without a default value. %d will be used.", (int32)DefaultValue);
		if (FPlatformMisc::IsDebuggerPresent())
		{
			UE_DEBUG_BREAK();
		}
		else
		{
			FDebug::DumpStackTraceToLog(ELogVerbosity::Error);
		}
	}

	return OpenModalMessageDialog_Internal(InMessageCategory, InMessageType, DefaultValue, InMessage, InTitle, ModalParent);
}

#undef LOCTEXT_NAMESPACE
