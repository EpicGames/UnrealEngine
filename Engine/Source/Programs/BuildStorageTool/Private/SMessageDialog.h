// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Internationalization/Text.h"

class SWindow;
class SWidget;

// Largely a copy of UnrealEd Dialogs.h

DECLARE_DELEGATE_TwoParams(FOnMsgDlgResult, const TSharedRef<SWindow>&, EAppReturnType::Type);

EAppReturnType::Type OpenModalMessageDialog_Internal(EAppMsgCategory InMessageCategory, EAppMsgType::Type InMessageType, const FText& InMessage, const FText& InTitle, const TSharedPtr<const SWidget>& ModalParent);
