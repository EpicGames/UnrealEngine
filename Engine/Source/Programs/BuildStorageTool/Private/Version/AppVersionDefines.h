// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Macros for encoding strings
#define VERSION_TEXT(x) TEXT(x)
#define VERSION_STRINGIFY_2(x) VERSION_TEXT(#x)
#define VERSION_STRINGIFY(x) VERSION_STRINGIFY_2(x)

#define BUILD_STORAGE_TOOL_VERSION_MAJOR 1
#define BUILD_STORAGE_TOOL_VERSION_MINOR 0
#define BUILD_STORAGE_TOOL_VERSION_PATCH 0

#define BUILD_STORAGE_TOOL_VERSION_STRING "1.00"

#define BUILD_STORAGE_TOOL_APPNAME TEXT("BuildStorageTool-Source")

#ifdef BUILD_STORAGE_TOOL_PRERELEASE
#undef BUILD_STORAGE_TOOL_APPNAME
#define BUILD_STORAGE_TOOL_APPNAME TEXT("BuildStorageTool-PreRelease")
#endif

#ifdef BUILD_STORAGE_TOOL_RELEASE
#undef BUILD_STORAGE_TOOL_APPNAME
#define BUILD_STORAGE_TOOL_APPNAME TEXT("BuildStorageTool")
#endif

#ifndef BUILD_STORAGE_TOOL_CHANGELIST
#define BUILD_STORAGE_TOOL_CHANGELIST_STRING TEXT("NoCL")
#endif

#ifdef BUILD_STORAGE_TOOL_CHANGELIST
#define BUILD_STORAGE_TOOL_CHANGELIST_STRING TEXT("CL-") VERSION_STRINGIFY(BUILD_STORAGE_TOOL_CHANGELIST)
#endif

